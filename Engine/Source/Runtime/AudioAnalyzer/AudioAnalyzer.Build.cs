// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class AudioAnalyzer : ModuleRules
	{
		public AudioAnalyzer(ReadOnlyTargetRules Target) : base(Target)
		{
			bUseUnity = true;

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"CoreUObject",
					"Engine",
					"AudioMixer"
				}
			);

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"SignalProcessing"
				}
			);

			if (Target.Type == TargetType.Editor && Target.Platform == UnrealTargetPlatform.Win64)
			{
				AddEngineThirdPartyPrivateStaticDependencies(Target, "UELibSampleRate");
			}
		}
	}
}
