// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/AsyncWork.h"
#include "AudioAnalyzerAsset.h"
#include "AudioAnalyzerFacade.h"
#include "AudioDefines.h"
#include "CoreMinimal.h"
#include "DSP/MultithreadedPatching.h"
#include "IAudioAnalyzerInterface.h"
#include "Sound/AudioBus.h"

#include "AudioAnalyzer.generated.h"

class UAudioAnalyzerSubsystem;

/** UAudioAnalyzerSettings
 *
 * UAudioAnalyzerSettings provides a way to store and reuse existing analyzer settings
 * across multiple analyzers. 
 *
 */
UCLASS(Abstract, EditInlineNew, BlueprintType, MinimalAPI)
class UAudioAnalyzerSettings : public UAudioAnalyzerAssetBase
{
	GENERATED_BODY()
};

typedef TSharedPtr<Audio::IAnalyzerResult, ESPMode::ThreadSafe> FAnalyzerResultSharedPtr;

class FAudioAnalyzeTask : public FNonAbandonableTask
{
public:
	FAudioAnalyzeTask(TUniquePtr<Audio::FAnalyzerFacade>& InAnalyzerFacade, int32 InSampleRate, int32 InNumChannels);
	
	// Set in the task the current state of the analyzer controls
	void SetAnalyzerControls(TSharedPtr<Audio::IAnalyzerControls> InControls);

	// Give the task the audio data to analyze
	void SetAudioBuffer(TArray<float>&& InAudioData);

	// Move the audio buffer back out
	TArray<float>&& GetAudioBuffer() { return MoveTemp(AudioData); }

	// Does the task work
	void DoWork();

	inline TStatId GetStatId() const { RETURN_QUICK_DECLARE_CYCLE_STAT(AudioAnalyzeTask, STATGROUP_ThreadPoolAsyncTasks); }

	// Get the results from the task
	TUniquePtr<Audio::IAnalyzerResult> GetResults() { return MoveTemp(Results); }

private:
	TUniquePtr<Audio::FAnalyzerFacade> AnalyzerFacade;
	int32 SampleRate = 0;
	int32 NumChannels = 0;
	TArray<float> AudioData;
	TUniquePtr<Audio::IAnalyzerResult> Results;
	TSharedPtr<Audio::IAnalyzerControls> AnalyzerControls;
};

/** UAudioAnalyzer
 *
 * UAudioAnalyzer performs analysis on an audio bus using specific settings and exposes the results via blueprints.
 *
 * Subclasses of UAudioAnalyzer must implement GetAnalyzerFactoryName() to associate
 * the UAudioAnalyzer asset with an IAudioAnalyzerFactory implementation.
 *
 * To support blueprint access, subclasses can implement UFUNCTIONs to expose the data
 * returned by GetResult().
 */
UCLASS(Abstract, EditInlineNew, BlueprintType, MinimalAPI)
class UAudioAnalyzer : public UObject
{
	GENERATED_BODY()
	
	friend class UAudioAnalyzerSubsystem;

public:
	/**
	 * ID to keep track of results. Useful for tracking most recent result when performing
	 * asynchronous processing.
	 */
	typedef int32 FResultId;

	/** Thread safe shared point to result object. */
	typedef TSharedPtr<Audio::IAnalyzerResult, ESPMode::ThreadSafe> FResultSharedPtr;

	/** The UAudioBus which is analyzed in real-time. */
	UPROPERTY(Transient)
	TObjectPtr<UAudioBus> AudioBus;

	/** Starts analyzing audio from the given audio bus. Optionally override the audio bus desired to analyze. */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = AudioAnalyzer, meta = (WorldContext = "WorldContextObject"))
	AUDIOANALYZER_API void StartAnalyzing(const UObject* WorldContextObject, UAudioBus* AudioBusToAnalyze);

	/** Starts analyzing using the given world.*/
	UE_DEPRECATED(5.4, "Use the StartAnalyzing method that uses Audio::FDeviceId.")
	AUDIOANALYZER_API void StartAnalyzing(UWorld* InWorld, UAudioBus* AudioBusToAnalyze);

	AUDIOANALYZER_API void StartAnalyzing(const Audio::FDeviceId InAudioDeviceId, UAudioBus* AudioBusToAnalyze);

	/** Stops analyzing audio. */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = AudioAnalyzer, meta = (WorldContext = "WorldContextObject"))
	AUDIOANALYZER_API void StopAnalyzing(const UObject* WorldContextObject = nullptr);

	/**
	 * Implementations can override this method to create settings objects
	 * specific for their analyzer.
	 */
	AUDIOANALYZER_API virtual TUniquePtr<Audio::IAnalyzerSettings> GetSettings(const int32 InSampleRate, const int32 InNumChannels) const;

	/**
	 * Implementations can override this method to create controls objects
	 * specific for their analyzer.
	 */
	AUDIOANALYZER_API virtual TSharedPtr<Audio::IAnalyzerControls> GetAnalyzerControls() const;

	/** Function to broadcast results. */
	virtual void BroadcastResults() {}

	//~ Begin UObject Interface.
	AUDIOANALYZER_API virtual void BeginDestroy() override;
	//~ End UObject interface

private: 
	/** Returns if this audio analyzer has enough audio queued up and is ready for analysis. */
	AUDIOANALYZER_API bool IsReadyForAnalysis() const;

	/** Does the actual analysis and casts the results to the derived classes type. Returns true if there results to broadcast. */
	AUDIOANALYZER_API bool DoAnalysis();

protected:

	template<class ResultType>
	TUniquePtr<ResultType> GetResults()
	{
		return TUniquePtr<ResultType>(static_cast<ResultType*>(ResultsInternal.Release()));
	}

	/* Subclasses must override this method in order to inform this object which AnalyzerFactory to use for analysis */
	virtual FName GetAnalyzerFactoryName() const PURE_VIRTUAL(UAudioAnalyzer::GetAnalyzerFactoryName, return FName(););

	/** How many frames of audio to wait before analyzing the audio. */
	int32 NumFramesPerBufferToAnalyze = 1024;

private:

	// Returns UAudioAnalyzerSettings* if property points to a valid UAudioAnalyzerSettings, otherwise returns nullptr.
	AUDIOANALYZER_API UAudioAnalyzerSettings* GetSettingsFromProperty(FProperty* Property);

	// Audio analysis subsystem used with this audio analyzer
	UPROPERTY(Transient)
	TObjectPtr<UAudioAnalyzerSubsystem> AudioAnalyzerSubsystem;

	// Output patch for retrieving audio from audio bus for analysis
	Audio::FPatchOutputStrongPtr PatchOutputStrongPtr;

	// Analysis task
	TSharedPtr<FAsyncTask<FAudioAnalyzeTask>> AnalysisTask;

	// The analyzer facade used for async tasks
	TUniquePtr<Audio::FAnalyzerFacade> AnalyzerFacade;

	// Cached results of previous analysis task. Can be invalid.
	TUniquePtr<Audio::IAnalyzerResult> ResultsInternal;

	// The analyzer controls that can be modified real-time
	TSharedPtr<Audio::IAnalyzerControls> AnalyzerControls;

	// Scratch buffer used for copying data from patch output and fed to analysis task
	TArray<float> AnalysisBuffer;

	// The sample of the audio renderer
	int32 AudioMixerSampleRate = 0;

	// The number of channels for the audio bus
	int32 NumBusChannels = 0;

	// The analyzer factory to use
	Audio::IAnalyzerFactory* AnalyzerFactory = nullptr;
};

