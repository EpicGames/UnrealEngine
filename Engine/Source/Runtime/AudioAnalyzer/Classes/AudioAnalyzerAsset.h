// Copyright Epic Games, Inc. All Rights Reserved.

/** Interface for Audio Analyzer Assets. */

#pragma once

#include "CoreMinimal.h"
#include "AudioAnalyzerAsset.generated.h"

DECLARE_MULTICAST_DELEGATE(FAnalyzeAudioDelegate);

/** UAudioAnalyzerAssetBase
 *
 * UAudioAnalyzerAssetBase provides the base interface for controlling asset actions within the editor.
 */
UCLASS(Abstract, EditInlineNew, MinimalAPI)
class UAudioAnalyzerAssetBase : public UObject
{
	GENERATED_BODY()

	public:

#if WITH_EDITOR
		virtual bool HasAssetActions() const { return true; }

		/**
		 * GetAssetActionName() returns the FText displayed in the editor.
		 */ 
		virtual FText GetAssetActionName() const PURE_VIRTUAL(UAudioAnalyzerAsset::GetAssetActionName, return FText(););

		/**
		 * GetSupportedClass() returns the class which should be associated with these asset actions.
		 */ 
		virtual UClass* GetSupportedClass() const PURE_VIRTUAL(UAudioAnalyzerAsset::GetSupportedClass, return nullptr;);

		/**
		 * GetTypeColor() returns the color used to display asset icons within the editor.
		 */
		virtual FColor GetTypeColor() const { return FColor(100.0f, 100.0f, 100.0f); }
#endif

};

