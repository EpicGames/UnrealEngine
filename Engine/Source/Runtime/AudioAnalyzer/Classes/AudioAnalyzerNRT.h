// Copyright Epic Games, Inc. All Rights Reserved.

/** Interface for Non Real-Time (NRT) Audio Analyzer UObjects. */

#pragma once

#include "CoreMinimal.h"
#include "IAudioAnalyzerNRTInterface.h"
#include "AudioAnalyzerAsset.h"
#include "Sound/SoundWave.h"
#include "AudioAnalyzerNRT.generated.h"



/** UAudioAnalyzerNRTSettings
 *
 * UAudioAnalyzerNRTSettings provides a way to store and reuse existing analyzer settings
 * across multiple analyzers. This class provides the interface and functionality to 
 * automatically trigger reanalysis of audio across all analyzers associated with this 
 * setting when when a UPROPERTY in this setting object is edited.
 *
 */
UCLASS(Abstract, EditInlineNew, BlueprintType, MinimalAPI)
class UAudioAnalyzerNRTSettings : public UAudioAnalyzerAssetBase
{
	GENERATED_BODY()

	public:

#if WITH_EDITOR

		FAnalyzeAudioDelegate AnalyzeAudioDelegate;

		/** 
		 * Called when a UPROPERTY of this class is edited. Triggers
		 * reanalysis of audio.
		 *
		 * This determines whether to trigger analysis by calling ShouldEventTriggerAnalysis(...)
		 */
		AUDIOANALYZER_API void PostEditChangeProperty (struct FPropertyChangedEvent & PropertyChangedEvent) override;

		/** 
		 * This returns true when the PropertyChangeEvent is due to update a setting property. 
		 * Override this method in order to customize this behavior.
		 */
		AUDIOANALYZER_API virtual bool ShouldEventTriggerAnalysis(struct FPropertyChangedEvent & PropertyChangeEvent);
#endif

};

/** Delegate to receive all meter results per channel (time-stamped in an array) since last delegate call. */
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnAudioAnalyzerNRTAnalysisComplete);

/** UAudioAnalyzerNRT
 *
 * UAudioAnalyzerNRT applies an analyzer to a sound using specific settings, stores the 
 * results and exposes them via blueprints. 
 *
 * Subclasses of UAudioAnalyzerNRT must implement GetAnalyzerNRTFactoryName() to associate
 * the UAudioAnalyzerNRT with an IAudioAnalyzerNRTFactory implementation. 
 *
 * To support blueprint access, subclasses can implement UFUNCTIONs to expose the data
 * returned by GetResult().
 */
UCLASS(Abstract, EditInlineNew, BlueprintType, MinimalAPI)
class UAudioAnalyzerNRT : public UAudioAnalyzerAssetBase
{
	GENERATED_BODY()

	public:
		/** 
		 * ID to keep track of results. Useful for tracking most recent result when performing
		 * asynchronous processing.
		 */
		typedef int32 FResultId;

		/** Thread safe shared point to result object. */
		typedef TSharedPtr<Audio::IAnalyzerNRTResult, ESPMode::ThreadSafe> FResultSharedPtr;

		/**
		 * The USoundWave which is analyzed.
		 */
		UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=AudioAnalyzer, meta=(DisallowedClasses="/Script/MetasoundEngine.MetaSoundSource, /Script/Engine.SoundSourceBus"))
		TObjectPtr<USoundWave> Sound;

		/** The duration of the analyzed audio in seconds. */
		UPROPERTY(BlueprintReadOnly, Category=AudioAnalyzer)
		float DurationInSeconds;

#if WITH_EDITOR
		UFUNCTION(BlueprintCallable, Category="Audio Analyzer")
		AUDIOANALYZER_API void SetSound(USoundWave* InSound);
#endif

		/**
		 * Returns the result object generated by the associated IAudioAnalyzerNRTFactory.
		 * The template argument must be the IAudioAnalyzerNRTResult subclass returned by
		 * the associated IAudioAnalyzerNRTFactory.
		 */
		template<class ResultType>
		TSharedPtr<ResultType, ESPMode::ThreadSafe> GetResult()
		{
			TSharedPtr<ResultType, ESPMode::ThreadSafe> ReturnedResult;
			{
				FScopeLock ResultLock(&ResultCriticalSection);
				ReturnedResult = StaticCastSharedPtr<ResultType>(Result); 
			}
			return ReturnedResult;
		}

		/**
		 * Returns the result object generated by the associated IAudioAnalyzerNRTFactory.
		 * The template argument must be the IAudioAnalyzerNRTResult subclass returned by
		 * the associated IAudioAnalyzerNRTFactory.
		 */
		template<class ResultType>
		TSharedPtr<const ResultType, ESPMode::ThreadSafe> GetResult() const
		{
			TSharedPtr<const ResultType, ESPMode::ThreadSafe> ReturnedResult;
			{
				FScopeLock ResultLock(&ResultCriticalSection);
				ReturnedResult = StaticCastSharedPtr<const ResultType>(Result); 
			}
			return ReturnedResult;
		}
	 
		/** 
		 * Implementations can override this method to create settings objects
		 * specific for their analyzer. 
		 */
		AUDIOANALYZER_API virtual TUniquePtr<Audio::IAnalyzerNRTSettings> GetSettings(const float InSampleRate, const int32 InNumChannels) const;

		/**
		 * Performs serialization of results.
		 */
		AUDIOANALYZER_API virtual void Serialize(FArchive& Ar) override;

#if WITH_EDITOR

		AUDIOANALYZER_API void SetResult(FResultSharedPtr NewResult);

		/** This will only store the result if the passed InResultId matches the CurrentResultId. */
		AUDIOANALYZER_API void SetResultIfLatest(FResultSharedPtr NewResult, FResultId InResultId);

		/**
		 * Called before a UPROPERTY of this class is edited. This checks to see
		 * if the UPROPERTY is a UAudioAnalyzerNRTSettings. If so, the previous settings' 
		 * AnalyzeAudioDelegate will be unbound from this object's AnalyzeAudio()
		 */
		AUDIOANALYZER_API void PreEditChange(FProperty* PropertyAboutToChange) override;

		/**
		 * Called when a UPROPERTY of this class is edited. Triggering 
		 * reanalysis of audio when appropriate and binds new settings if the UPROPERTY
		 * is a UAudioAnalyzerNRTSettings derived object.
		 *
		 * This determines whether to trigger analysis by calling ShouldEventTriggerAnalysis(...)
		 */
		AUDIOANALYZER_API void PostEditChangeProperty (struct FPropertyChangedEvent & PropertyChangedEvent) override;

		/**
		 * Returns true when the PropertyChangeEvent is due to update SoundWave or Settings.
		 */
		AUDIOANALYZER_API virtual bool ShouldEventTriggerAnalysis(struct FPropertyChangedEvent & PropertyChangeEvent);

		/** Performs the analaysis of the audio */
		UFUNCTION(BlueprintCallable, Category="Audio Analyzer")
		AUDIOANALYZER_API void AnalyzeAudio();

#endif
#if WITH_EDITORONLY_DATA
		/** Delegate to receive all meter results, per-channel, since last delegate call. */
		UPROPERTY(BlueprintAssignable, Category="Audio Analyzer")
		FOnAudioAnalyzerNRTAnalysisComplete OnAnalysisComplete;
#endif


	protected:

#if WITH_EDITOR

		/** This sets the AnalyzeAudio callback to the delegate in settings. */
		AUDIOANALYZER_API void SetSettingsDelegate(UAudioAnalyzerNRTSettings* InSettings);

		/** This removes the existing delegate handle from the current settings object */
		AUDIOANALYZER_API void RemoveSettingsDelegate(UAudioAnalyzerNRTSettings* InSettings);
#endif 

		/* Subclasses must override this method in order to inform this object which AnalyzerNRTFactory to use for analysis */
		virtual FName GetAnalyzerNRTFactoryName() const PURE_VIRTUAL(UAudioAnalyzerNRT::GetAnalyzerNRTFactoryName, return FName(););

	private:

		// Returns UAudioAnalyzerNRTSettings* if property points to a valid UAudioAnalyzerNRTSettings, otherwise returns nullptr.
		AUDIOANALYZER_API UAudioAnalyzerNRTSettings* GetSettingsFromProperty(FProperty* Property);

		TSharedPtr<Audio::IAnalyzerNRTResult, ESPMode::ThreadSafe> Result;

		// Critical section is mutable so it can still be locked within const methods.
		mutable FCriticalSection ResultCriticalSection;

		// Result id of the current result. 
		TAtomic<FResultId> CurrentResultId;

#if WITH_EDITOR
		TMap<UAudioAnalyzerNRTSettings*, FDelegateHandle> AnalyzeAudioDelegateHandles;
#endif 
};

