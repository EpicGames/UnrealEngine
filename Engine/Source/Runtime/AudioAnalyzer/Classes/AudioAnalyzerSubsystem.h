// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AudioAnalyzer.h"
#include "Containers/Ticker.h"
#include "Subsystems/EngineSubsystem.h"
#include "AudioAnalyzerSubsystem.generated.h"

class UWorld;

/** 
* Class manages querying analysis results from various audio analyzers.
*/
UCLASS(MinimalAPI)
class UAudioAnalyzerSubsystem : public UEngineSubsystem
{
	GENERATED_BODY()

public:
	AUDIOANALYZER_API UAudioAnalyzerSubsystem();
	AUDIOANALYZER_API ~UAudioAnalyzerSubsystem();

	AUDIOANALYZER_API bool Tick(float DeltaTime);

	//~ Begin USubsystem
	AUDIOANALYZER_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	AUDIOANALYZER_API virtual void Deinitialize() override;
	//~ End USubsystem

	AUDIOANALYZER_API void RegisterAudioAnalyzer(UAudioAnalyzer* InAnalyzer);
	AUDIOANALYZER_API void UnregisterAudioAnalyzer(UAudioAnalyzer* InAnalyzer);

	static AUDIOANALYZER_API UAudioAnalyzerSubsystem* Get();

private:

	UPROPERTY(Transient);
	TArray<TObjectPtr<UAudioAnalyzer>> AudioAnalyzers;

	FTSTicker::FDelegateHandle TickerHandle;
};
