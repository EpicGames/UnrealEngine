// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioAnalyzer.h"

#include "Async/Async.h"
#include "AudioAnalyzerFacade.h"
#include "AudioAnalyzerModule.h"
#include "AudioAnalyzerSubsystem.h"
#include "AudioBusSubsystem.h"
#include "AudioDeviceHandle.h"
#include "AudioDeviceManager.h"
#include "AudioMixerDevice.h"
#include "Engine/Engine.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioAnalyzer)


FAudioAnalyzeTask::FAudioAnalyzeTask(TUniquePtr<Audio::FAnalyzerFacade>& InAnalyzerFacade, int32 InSampleRate, int32 InNumChannels)
	: AnalyzerFacade(InAnalyzerFacade.Release())
	, SampleRate(InSampleRate)
	, NumChannels(InNumChannels)
{
}

void FAudioAnalyzeTask::SetAudioBuffer(TArray<float>&& InAudioData)
{
	AudioData = MoveTemp(InAudioData);
}

void FAudioAnalyzeTask::SetAnalyzerControls(TSharedPtr<Audio::IAnalyzerControls> InControls)
{
	AnalyzerControls = InControls;
}

void FAudioAnalyzeTask::DoWork()
{
	check(AnalyzerFacade);
	Results = AnalyzerFacade->AnalyzeAudioBuffer(AudioData, NumChannels, SampleRate, AnalyzerControls);
}

void UAudioAnalyzer::StartAnalyzing(UWorld* InWorld, UAudioBus* AudioBusToAnalyze)
{
	if (!InWorld)
	{
		return;
	}

	const FAudioDeviceHandle AudioDevice = InWorld->GetAudioDevice();
	if (!AudioDevice.IsValid())
	{
		return;
	}

	StartAnalyzing(AudioDevice.GetDeviceID(), AudioBusToAnalyze);
}

void UAudioAnalyzer::StartAnalyzing(const Audio::FDeviceId InAudioDeviceId, UAudioBus* AudioBusToAnalyze)
{
	if (!AudioBusToAnalyze)
	{
		UE_LOGF(LogAudioAnalyzer, Error, "Unable to analyze audio without an audio bus to analyze.");
		return;
	}

	const FAudioDeviceManager* AudioDeviceManager = FAudioDeviceManager::Get();
	if (!AudioDeviceManager)
	{
		UE_LOGF(LogAudioAnalyzer, Error, "Unable to analyze audio with a null audio device manager.");
		return;
	}

	const Audio::FMixerDevice* MixerDevice = static_cast<const Audio::FMixerDevice*>(AudioDeviceManager->GetAudioDeviceRaw(InAudioDeviceId));

	if (!MixerDevice)
	{
		UE_LOGF(LogAudioAnalyzer, Error, "Audio analyzer only works with the audio mixer.");
		return;
	}

	// Retrieve the audio analyzer subsystem if it's not already retrieved
	if (!AudioAnalyzerSubsystem)
	{
		AudioAnalyzerSubsystem = UAudioAnalyzerSubsystem::Get();
	}

	// Store the audio bus we're analyzing
	AudioBus = AudioBusToAnalyze;

	NumBusChannels = AudioBus->GetNumChannels();
	check(NumBusChannels > 0);

	AudioMixerSampleRate = (int32)MixerDevice->GetSampleRate();

	// Get the analyzer factory to use for our analysis
	if (!AnalyzerFactory)
	{
		AnalyzerFactory = Audio::GetAnalyzerFactory(GetAnalyzerFactoryName());
	}
	checkf(AnalyzerFactory != nullptr, TEXT("Need to register the factory as a modular feature for the analyzer '%s'."), *GetAnalyzerFactoryName().ToString());

	// Start the audio bus. This won't do anything if the bus is already started elsewhere.
	uint32 AudioBusId = AudioBus->GetUniqueID();
	int32 NumChannels = (int32)AudioBus->AudioBusChannels + 1;

	UAudioBusSubsystem* AudioBusSubsystem = MixerDevice->GetSubsystem<UAudioBusSubsystem>();
	check(AudioBusSubsystem);
	Audio::FAudioBusKey AudioBusKey = Audio::FAudioBusKey(AudioBusId);

	const FString AudioAnalyzerBusName = FString::Format(TEXT("_Analyzer_{0}_AudioBusId_{1}"), { *GetName(), AudioBusId });
	AudioBusSubsystem->StartAudioBus(AudioBusKey, AudioAnalyzerBusName, NumChannels, false);

	// Get an output patch for the audio bus
	NumFramesPerBufferToAnalyze = MixerDevice->GetNumOutputFrames();
	PatchOutputStrongPtr = AudioBusSubsystem->AddPatchOutputForAudioBus(AudioBusKey, NumFramesPerBufferToAnalyze, NumChannels);

	// Register this audio analyzer with the audio analyzer subsystem
	// The subsystem will query this analyzer to see if it has enough audio to perform analysis.
	// If it does, it'll report the results (of any previous analysis) and ask us to start analyzing the audio.	
	if (AudioAnalyzerSubsystem)
	{
		AudioAnalyzerSubsystem->RegisterAudioAnalyzer(this);
	}

	// Setup the analyzer facade here once
	AnalyzerFacade = MakeUnique<Audio::FAnalyzerFacade>(GetSettings(AudioMixerSampleRate, NumBusChannels), AnalyzerFactory);
}


void UAudioAnalyzer::StartAnalyzing(const UObject* WorldContextObject, UAudioBus* AudioBusToAnalyze)
{
	if (!AudioBusToAnalyze)
	{
		UE_LOGF(LogAudioAnalyzer, Error, "Unable to analyze audio without an audio bus to analyze.");
		return;
	}

	// Retrieve the world and if it's not a valid world, don't do anything
	UWorld* ThisWorld = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::ReturnNull);
	if (!ThisWorld || !ThisWorld->bAllowAudioPlayback || ThisWorld->GetNetMode() == NM_DedicatedServer)
	{
		return;
	}

	const FAudioDeviceHandle AudioDevice = ThisWorld->GetAudioDevice();
	if (!AudioDevice.IsValid())
	{
		return;
	}

	StartAnalyzing(AudioDevice.GetDeviceID(), AudioBusToAnalyze);
}

void UAudioAnalyzer::StopAnalyzing(const UObject* WorldContextObject)
{
	if (AudioAnalyzerSubsystem)
	{
		// Unregister the audio analyzer from the analyzer subsystem
		AudioAnalyzerSubsystem->UnregisterAudioAnalyzer(this);

		// Null out our patch output
		PatchOutputStrongPtr = nullptr;
	}
}

bool UAudioAnalyzer::IsReadyForAnalysis() const
{
	check(AudioBus != nullptr);

	// Only allow one worker task to happen at a time
	if (AnalysisTask.IsValid() && !AnalysisTask->IsWorkDone())
	{
		return false;
	}

	// Only ready if we've got a patch output and there's enough audio queued up
	if (PatchOutputStrongPtr.IsValid())
	{
		int32 NumSamplesAvailable = PatchOutputStrongPtr->GetNumSamplesAvailable();
		int32 NumAudioBusChannels = AudioBus->GetNumChannels();
		check(NumAudioBusChannels > 0);
		int32 NumFramesAvailable = NumSamplesAvailable / NumAudioBusChannels;
		return NumFramesAvailable >= NumFramesPerBufferToAnalyze;
	}
	return false;
}

bool UAudioAnalyzer::DoAnalysis()
{
	bool bHasResults = false;

	// Make sure the task is finished before we do any more analysis
	if (AnalysisTask.IsValid())
	{
		check(AnalysisTask->IsWorkDone());

		// Do final ensure completion before reusing the task
		AnalysisTask->EnsureCompletion();

		// Report results from previous analysis
		FAudioAnalyzeTask& Task = AnalysisTask->GetTask();
		ResultsInternal = Task.GetResults();

		// Retrieve the analysis buffer so we can reuse the memory when we copy to it from the patch output
		AnalysisBuffer = Task.GetAudioBuffer();

		bHasResults = true;
	}
	else
	{
		// Create a task if one doesn't exist
		AnalysisTask = TSharedPtr<FAsyncTask<FAudioAnalyzeTask>>(new FAsyncTask<FAudioAnalyzeTask>(AnalyzerFacade, AudioMixerSampleRate, NumBusChannels));
	}

	// Make sure our task is valid and done
	check(AnalysisTask.IsValid() && AnalysisTask->IsDone());

	// Copy the audio to be analyzed from the patch output
	int32 NumSamplesAvailable = PatchOutputStrongPtr->GetNumSamplesAvailable();

	AnalysisBuffer.Reset();
	AnalysisBuffer.AddUninitialized(NumSamplesAvailable);
	PatchOutputStrongPtr->PopAudio(AnalysisBuffer.GetData(), NumSamplesAvailable, false);

	check(AudioMixerSampleRate != 0);
	check(NumBusChannels != 0);

	FAudioAnalyzeTask& Task = AnalysisTask->GetTask();
	Task.SetAnalyzerControls(GetAnalyzerControls());
	Task.SetAudioBuffer(MoveTemp(AnalysisBuffer));

 	AnalysisTask->StartBackgroundTask();

	return bHasResults;
}

void UAudioAnalyzer::BeginDestroy()
{
	Super::BeginDestroy();

	if (AnalysisTask.IsValid())
	{
		AnalysisTask->EnsureCompletion();
	}

	// Unregister this audio analyzer from the subsystem since we won't be needing anymore ticks
	if (AudioAnalyzerSubsystem)
	{
		AudioAnalyzerSubsystem->UnregisterAudioAnalyzer(this);
	}
}

UAudioAnalyzerSettings* UAudioAnalyzer::GetSettingsFromProperty(FProperty* Property)
{
	if (nullptr == Property)
	{
		return nullptr;
	}

	if (Property->IsA(FObjectPropertyBase::StaticClass()))
	{
		FObjectPropertyBase* ObjectPropertyBase = CastFieldChecked<FObjectPropertyBase>(Property);

		if (nullptr == ObjectPropertyBase)
		{
			return nullptr;
		}

		if (ObjectPropertyBase->PropertyClass->IsChildOf(UAudioAnalyzerSettings::StaticClass()))
		{
			UObject* PropertyObject = ObjectPropertyBase->GetObjectPropertyValue_InContainer(this);
			return Cast<UAudioAnalyzerSettings>(PropertyObject);
		}
	}

	return nullptr;
}

TUniquePtr<Audio::IAnalyzerSettings> UAudioAnalyzer::GetSettings(const int32 InSampleRate, const int32 InNumChannels) const
{
	return MakeUnique<Audio::IAnalyzerSettings>();
}

TSharedPtr<Audio::IAnalyzerControls> UAudioAnalyzer::GetAnalyzerControls() const
{
	return AnalyzerControls;
}
