// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioAnalyzerFacade.h"

#include "AudioAnalyzerModule.h"
#include "IAudioAnalyzerInterface.h"
#include "Features/IModularFeatures.h"

namespace Audio
{
	IAnalyzerFactory* GetAnalyzerFactory(FName InFactoryName)
	{
		AUDIO_ANALYSIS_LLM_SCOPE

		// Get all analyzer nrt factories implementations.
		IModularFeatures::Get().LockModularFeatureList();
		TArray<IAnalyzerFactory*> RegisteredFactories = IModularFeatures::Get().GetModularFeatureImplementations<IAnalyzerFactory>(IAnalyzerFactory::GetModularFeatureName());
		IModularFeatures::Get().UnlockModularFeatureList();

		// Get the factory of interest by matching the name.
		TArray<IAnalyzerFactory*> MatchingFactories = RegisteredFactories.FilterByPredicate([InFactoryName](IAnalyzerFactory* Factory) { check(nullptr != Factory); return Factory->GetName() == InFactoryName; });

		if (0 == MatchingFactories.Num())
		{
			// There is a likely programming error if the factory is not found. 
			UE_LOGF(LogAudioAnalyzer, Error, "Failed to find factory of type '%ls' with name '%ls'", *IAnalyzerFactory::GetModularFeatureName().ToString(), *InFactoryName.ToString());

			return nullptr;
		}

		if (MatchingFactories.Num() > 1)
		{
			// If multiple factories with the same name exist, the first one in the array will be used. 
			UE_LOGF(LogAudioAnalyzer, Warning, "Found multiple factories of type '%ls' with name '%ls'. Factory names should be unique.", *IAnalyzerFactory::GetModularFeatureName().ToString(), *InFactoryName.ToString());
		}

		return MatchingFactories[0];
	}

	FAnalyzerFacade::FAnalyzerFacade(TUniquePtr<IAnalyzerSettings> InSettings, IAnalyzerFactory* InFactory)
		: Settings(MoveTemp(InSettings))
		, Factory(InFactory)
	{
	}

	TUniquePtr<IAnalyzerResult> FAnalyzerFacade::AnalyzeAudioBuffer(const TArray<float>& InAudioBuffer, int32 InNumChannels, float InSampleRate, TSharedPtr<IAnalyzerControls> InControls)
	{
		LLM_SCOPE_BYTAG(Audio_Analysis);

		if (nullptr == Factory)
		{
			UE_LOGF(LogAudioAnalyzer, Error, "Cannot analyze audio due to null factory");

			return TUniquePtr<IAnalyzerResult>();
		}

		// Create result and worker from factory
		if (!Worker.IsValid())
		{
			FAnalyzerParameters AnalyzerParameters(InSampleRate, InNumChannels);
			Worker = Factory->NewWorker(AnalyzerParameters, Settings.Get());
		}

		// Check that worker created successfully
		if (!Worker.IsValid())
		{
			UE_LOGF(LogAudioAnalyzer, Error, "Failed to create IAnalyzerWorker with factory of type '%ls' with name '%ls'", *IAnalyzerFactory::GetModularFeatureName().ToString(), *Factory->GetName().ToString());

			return TUniquePtr<IAnalyzerResult>();
		}

		TUniquePtr<IAnalyzerResult> Result = Factory->NewResult();

		// Check that result created successfully
		if (!Result.IsValid())
		{
			UE_LOGF(LogAudioAnalyzer, Error, "Failed to create IAnalyzerResult with factory of type '%ls' with name '%ls'", *Audio::IAnalyzerFactory::GetModularFeatureName().ToString(), *Factory->GetName().ToString());

			return TUniquePtr<IAnalyzerResult>();
		}

		// Perform and finalize audio analysis.
		Worker->SetControls(InControls);
		Worker->Analyze(MakeArrayView(InAudioBuffer), Result.Get());

		return Result;
	}
}

