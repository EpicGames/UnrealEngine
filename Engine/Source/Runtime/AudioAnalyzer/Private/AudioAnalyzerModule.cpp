// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioAnalyzerModule.h"

#include "HAL/LowLevelMemStats.h"
#include "HAL/LowLevelMemTracker.h"
#include "Logging/LogMacros.h"

DEFINE_LOG_CATEGORY(LogAudioAnalyzer);

DECLARE_LLM_MEMORY_STAT(TEXT("AudioAnalysis"), STAT_AudioAnalysisLLM, STATGROUP_LLMFULL);
LLM_DEFINE_TAG(Audio_Analysis, TEXT("Audio Analysis"), TEXT("Audio"), GET_STATFNAME(STAT_AudioAnalysisLLM), GET_STATFNAME(STAT_AudioSummaryLLM));

void FAudioAnalyzerModule::StartupModule()
{
}

void FAudioAnalyzerModule::ShutdownModule()
{
}

IMPLEMENT_MODULE(FAudioAnalyzerModule, AudioAnalyzer);
