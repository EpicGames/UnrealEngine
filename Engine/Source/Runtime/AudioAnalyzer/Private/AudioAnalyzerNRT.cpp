// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioAnalyzerNRT.h"
#include "AudioAnalyzerNRTFacade.h"
#include "AudioAnalyzerModule.h"
#include "SampleBuffer.h"
#include "Async/Async.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioAnalyzerNRT)
#if WITH_EDITOR

namespace 
{
	class FAudioAnalyzeNRTTask : public FNonAbandonableTask
	{
		friend class FAutoDeleteAsyncTask<FAudioAnalyzeNRTTask>;

		public:
			FAudioAnalyzeNRTTask(
					TWeakObjectPtr<UAudioAnalyzerNRT> InAnalyzerUObject, 
					const UAudioAnalyzerNRT::FResultId InResultId,
					TUniquePtr<Audio::FAnalyzerNRTFacade>&& InAnalyzerFacade, 
					TArray<uint8>&& InRawWaveData,
					int32 InNumChannels,
					float InSampleRate)
			: AnalyzerUObject(InAnalyzerUObject)
			, ResultId(InResultId)
			, AnalyzerFacade(MoveTemp(InAnalyzerFacade))
			, RawWaveData(MoveTemp(InRawWaveData))
			, NumChannels(InNumChannels)
			, SampleRate(InSampleRate)
			{}

			void DoWork()
			{
				TUniquePtr<Audio::IAnalyzerNRTResult> Result = AnalyzerFacade->AnalyzePCM16Audio(RawWaveData, NumChannels, SampleRate);

				UAudioAnalyzerNRT::FResultSharedPtr ResultPtr(Result.Release());
				// Set value on game thread.
				AsyncTask(ENamedThreads::GameThread, [Analyzer = AnalyzerUObject, ThisResultId = ResultId, ResultPtr]() {
					if (Analyzer.IsValid())
					{
						Analyzer->SetResultIfLatest(ResultPtr, ThisResultId);
					}
				});
			}

			FORCEINLINE TStatId GetStatId() const { RETURN_QUICK_DECLARE_CYCLE_STAT(AudioAnalyzeNRTTask, STATGROUP_ThreadPoolAsyncTasks); }

		private:
			TWeakObjectPtr<UAudioAnalyzerNRT> AnalyzerUObject;
			const UAudioAnalyzerNRT::FResultId ResultId;
			TUniquePtr<Audio::FAnalyzerNRTFacade> AnalyzerFacade;
			TArray<uint8> RawWaveData;
			int32 NumChannels;
			float SampleRate;
	};
}

/*****************************************************/
/*********** UAudioAnalyzerNRTSettings ***************/
/*****************************************************/

void UAudioAnalyzerNRTSettings::PostEditChangeProperty (struct FPropertyChangedEvent & PropertyChangedEvent) 
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (ShouldEventTriggerAnalysis(PropertyChangedEvent))
	{
		AnalyzeAudioDelegate.Broadcast();
	}
}

bool UAudioAnalyzerNRTSettings::ShouldEventTriggerAnalysis(struct FPropertyChangedEvent & PropertyChangeEvent)
{
	// By default, all non-interactive changes to settings will trigger analysis.
	return PropertyChangeEvent.ChangeType != EPropertyChangeType::Interactive;
}


/*****************************************************/
/***********      UAudioAnalyzerNRT    ***************/
/*****************************************************/
void UAudioAnalyzerNRT::SetSound(USoundWave* InSound)
{
	Sound = InSound;
}

void UAudioAnalyzerNRT::PreEditChange(FProperty* PropertyAboutToChange)
{
	// If the settings object is replaced, need to unbind any existing settings objects
	// from calling the analyze audio delegate.
	Super::PreEditChange(PropertyAboutToChange);
	
	UAudioAnalyzerNRTSettings* Settings = GetSettingsFromProperty(PropertyAboutToChange);

	if (Settings)
	{
		RemoveSettingsDelegate(Settings);
	}
}

void UAudioAnalyzerNRT::PostEditChangeProperty (struct FPropertyChangedEvent & PropertyChangedEvent) 
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	
	// Check if the edited property was a UAudioAnalyzerNRTSettings object
	UAudioAnalyzerNRTSettings* Settings = GetSettingsFromProperty(PropertyChangedEvent.Property);

	if (Settings)
	{
		// If it was a UAudioAnalyzerNRTSettings object, bind the FAnalyzeAudioDelegate
		SetSettingsDelegate(Settings);
	}

	if (ShouldEventTriggerAnalysis(PropertyChangedEvent))
	{
		AnalyzeAudio();
	}
}

bool UAudioAnalyzerNRT::ShouldEventTriggerAnalysis(struct FPropertyChangedEvent & PropertyChangeEvent)
{
	// by default, all changes will trigger analysis
	return true;
}

void UAudioAnalyzerNRT::AnalyzeAudio()
{
	AUDIO_ANALYSIS_LLM_SCOPE
	
	check(IsInGameThread());

	// Create a new result id for this result.
	FResultId ThisResultId = ++CurrentResultId;

	if (nullptr != Sound)
	{
		if (Sound->bProcedural)
		{
			UE_LOGF(LogAudioAnalyzer, Warning, "Soundwave '%ls' is procedural. NRT audio analysis is not currently supported for this.", *Sound->GetFullName());
			SetResult(nullptr);
			return;
		}

		// Read audio while Sound object is assured safe. 
		if (Sound->ChannelSizes.Num() > 0)
		{
			UE_LOGF(LogAudioAnalyzer, Warning, "Soundwave '%ls' has multi-channel audio (channels greater than 2). Audio analysis is not currently supported for this yet.", *Sound->GetFullName());
			SetResult(nullptr);
			return;
		}

		// Retrieve the raw imported data
		TArray<uint8> RawWaveData;
		uint32 SampleRate = 0;
		uint16 NumChannels = 0;

		if (!Sound->GetImportedSoundWaveData(RawWaveData, SampleRate, NumChannels))
		{
			UE_LOGF(LogAudioAnalyzer, Error, "Could not analyze audio due to failed import of sound wave data from Soundwave '%ls'.", *Sound->GetFullName());
			SetResult(nullptr);
			return;
		}

		if (SampleRate == 0 || NumChannels == 0)
		{
			UE_LOGF(LogAudioAnalyzer, Error, "Failed to parse the raw imported data for '%ls' for analysis.", *Sound->GetFullName());
			SetResult(nullptr);
			return;
		}
		
		// Create analyzer helper object
		TUniquePtr<Audio::FAnalyzerNRTFacade> BatchAnalyzer = MakeUnique<Audio::FAnalyzerNRTFacade>(GetSettings(SampleRate, NumChannels), GetAnalyzerNRTFactoryName());

		// Use weak reference in case this object is deleted before analysis is done
		TWeakObjectPtr<UAudioAnalyzerNRT> AnalyzerPtr(this);
		
		// Create and start async task. Parentheses avoids memory leak warnings from static analysis.
		(new FAutoDeleteAsyncTask<FAudioAnalyzeNRTTask>(AnalyzerPtr, ThisResultId, MoveTemp(BatchAnalyzer), MoveTemp(RawWaveData), NumChannels, SampleRate))->StartBackgroundTask();
	}
	else
	{
		// Copy empty result to this object
		SetResult(nullptr);
	}
}

// Returns UAudioAnalyzerNRTSettings* if property points to a valid UAudioAnalyzerNRTSettings, otherwise returns nullptr.
UAudioAnalyzerNRTSettings* UAudioAnalyzerNRT::GetSettingsFromProperty(FProperty* Property)
{
	if (nullptr == Property)
	{
		return nullptr;
	}

	if (Property->IsA(FObjectPropertyBase::StaticClass()))
	{
		FObjectPropertyBase* ObjectPropertyBase = CastFieldChecked<FObjectPropertyBase>(Property);
		
		if (nullptr == ObjectPropertyBase)
		{
			return nullptr;
		}

		if (ObjectPropertyBase->PropertyClass->IsChildOf(UAudioAnalyzerNRTSettings::StaticClass()))
		{
			UObject* PropertyObject = ObjectPropertyBase->GetObjectPropertyValue_InContainer(this);
			return Cast<UAudioAnalyzerNRTSettings>(PropertyObject);
		}
	}

	return nullptr;
}

void UAudioAnalyzerNRT::SetResult(FResultSharedPtr NewResult)
{
	check(IsInGameThread());

	
	FScopeLock ResultLock(&ResultCriticalSection);
	Result = NewResult;
	if (Result.IsValid())
	{
		DurationInSeconds = Result->GetDurationInSeconds();
	}
	else
	{
		DurationInSeconds = 0.f;
	}
	Modify();
#if WITH_EDITORONLY_DATA
	OnAnalysisComplete.Broadcast();
#endif
	

}

void UAudioAnalyzerNRT::SetResultIfLatest(FResultSharedPtr NewResult, FResultId InResultId)
{
	
	FScopeLock ResultLock(&ResultCriticalSection);
	const FResultId ResultId = CurrentResultId.Load();

	if (ResultId == InResultId)
	{
		Result = NewResult;
		if (Result.IsValid())
		{
			DurationInSeconds = Result->GetDurationInSeconds();
		}
		else
		{
			DurationInSeconds = 0.f;
		}
		Modify();
#if WITH_EDITORONLY_DATA
		OnAnalysisComplete.Broadcast();
#endif
	}
}

#endif


void UAudioAnalyzerNRT::Serialize(FArchive& Ar)
{
	// default uobject serialize
	Super::Serialize(Ar);

	// When loading object, Result pointer is invalid. Need to create a valid 
	// result object for loading.
	if (!Result.IsValid())
	{
		if (!GetClass()->HasAnyClassFlags(CLASS_Abstract))
		{
			Audio::IAnalyzerNRTFactory* Factory = Audio::GetAnalyzerNRTFactory(GetAnalyzerNRTFactoryName());

			if (nullptr != Factory)
			{
				// Create result and worker from factory
				{
					FScopeLock ResultLock(&ResultCriticalSection);
					Result = Factory->NewResultShared<ESPMode::ThreadSafe>();
				}
			}
		}
	}

	if (Result.IsValid())
	{
		FScopeLock ResultLock(&ResultCriticalSection);
		Result->Serialize(Ar);
	}
}

TUniquePtr<Audio::IAnalyzerNRTSettings> UAudioAnalyzerNRT::GetSettings(const float InSampleRate, const int32 InNumChannels) const
{
	AUDIO_ANALYSIS_LLM_SCOPE

	return MakeUnique<Audio::IAnalyzerNRTSettings>();
}

#if WITH_EDITOR
void UAudioAnalyzerNRT::SetSettingsDelegate(UAudioAnalyzerNRTSettings* InSettings)
{
	if (InSettings)
	{
		if (AnalyzeAudioDelegateHandles.Contains(InSettings))
		{
			// Avoid setting delegate more tha once
			return;
		}

		FDelegateHandle DelegateHandle = InSettings->AnalyzeAudioDelegate.AddUObject(this, &UAudioAnalyzerNRT::AnalyzeAudio);

		if (DelegateHandle.IsValid())
		{
			AnalyzeAudioDelegateHandles.Add(InSettings, DelegateHandle);
		}
	}
}

void UAudioAnalyzerNRT::RemoveSettingsDelegate(UAudioAnalyzerNRTSettings* InSettings)
{
	if (InSettings)
	{
		if (AnalyzeAudioDelegateHandles.Contains(InSettings))
		{
			FDelegateHandle DelegateHandle = AnalyzeAudioDelegateHandles[InSettings];

			if (DelegateHandle.IsValid())
			{
				InSettings->AnalyzeAudioDelegate.Remove(DelegateHandle);
			}

			AnalyzeAudioDelegateHandles.Remove(InSettings);
		}
	}
}

#endif
