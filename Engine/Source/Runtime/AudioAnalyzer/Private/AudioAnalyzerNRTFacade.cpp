// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioAnalyzerNRTFacade.h"

#include "AudioAnalyzerModule.h"
#include "IAudioAnalyzerNRTInterface.h"
#include "Features/IModularFeatures.h"
#include "SampleBuffer.h"

namespace Audio
{
	IAnalyzerNRTFactory* GetAnalyzerNRTFactory(FName InFactoryName)
	{
		// Get all analyzer nrt factories implementations.
		IModularFeatures::Get().LockModularFeatureList();
		TArray<IAnalyzerNRTFactory*> RegisteredFactories = IModularFeatures::Get().GetModularFeatureImplementations<IAnalyzerNRTFactory>(IAnalyzerNRTFactory::GetModularFeatureName());
		IModularFeatures::Get().UnlockModularFeatureList();

		// Get the factory of interest by matching the name.
		TArray<IAnalyzerNRTFactory*> MatchingFactories = RegisteredFactories.FilterByPredicate([InFactoryName](IAnalyzerNRTFactory* Factory) { check(nullptr != Factory); return Factory->GetName() == InFactoryName; });

		if (0 == MatchingFactories.Num())
		{
			// There is a likely programming error if the factory is not found. 
			UE_LOGF(LogAudioAnalyzer, Error, "Failed to find factory of type '%ls' with name '%ls'", *IAnalyzerNRTFactory::GetModularFeatureName().ToString(), *InFactoryName.ToString());

			return nullptr;
		}

		if (MatchingFactories.Num() > 1)
		{
			// Like the Highlander, there should be only one. If multiple factories with the same name exist, the first one in the array will be used. 
			UE_LOGF(LogAudioAnalyzer, Warning, "Found multiple factories of type '%ls' with name '%ls'. Factory names should be unique.", *IAnalyzerNRTFactory::GetModularFeatureName().ToString(), *InFactoryName.ToString());
		}

		return MatchingFactories[0];
	}

	/**********************************************************/
	/*************** FAnalyzerNRTFacade ************************/
	/**********************************************************/

	/**
	 * Create an FAnalyzerNRTFacade with the analyzer settings and factory name.
	 */
	FAnalyzerNRTFacade::FAnalyzerNRTFacade(TUniquePtr<IAnalyzerNRTSettings> InSettings, const FName& InFactoryName)
	: Settings(MoveTemp(InSettings))
	, FactoryName(InFactoryName)
	{}

	/**
	 * Analyze an entire PCM16 encoded audio object.  Audio for the entire sound should be contained within InRawWaveData.
	 */
	TUniquePtr<IAnalyzerNRTResult> FAnalyzerNRTFacade::AnalyzePCM16Audio(const TArray<uint8>& InRawWaveData, int32 InNumChannels, float InSampleRate)
	{
		AUDIO_ANALYSIS_LLM_SCOPE

		// Get analyzer factory
		IAnalyzerNRTFactory* Factory = GetAnalyzerNRTFactory(FactoryName);

		if (nullptr == Factory)
		{
			UE_LOGF(LogAudioAnalyzer, Error, "Cannot analyze audio due to null factory");

			return TUniquePtr<IAnalyzerNRTResult>();
		}

		// Create result and worker from factory
		FAnalyzerNRTParameters AnalyzerNRTParameters(InSampleRate, InNumChannels);

		TUniquePtr<IAnalyzerNRTResult> Result = Factory->NewResult();
		TUniquePtr<IAnalyzerNRTWorker> Worker = Factory->NewWorker(AnalyzerNRTParameters, Settings.Get());

		// Check that worker created successfully
		if (!Worker.IsValid())
		{
			UE_LOGF(LogAudioAnalyzer, Error, "Failed to create IAnalyzerNRTWorker with factory of type '%ls' with name '%ls'", *IAnalyzerNRTFactory::GetModularFeatureName().ToString(), *Factory->GetName().ToString());

			return TUniquePtr<IAnalyzerNRTResult>();
		}

		// Check that result created successfully
		if (!Result.IsValid())
		{
			UE_LOGF(LogAudioAnalyzer, Error, "Failed to create IAnalyzerNRTResult with factory of type '%ls' with name '%ls'", *Audio::IAnalyzerNRTFactory::GetModularFeatureName().ToString(), *Factory->GetName().ToString());

			return TUniquePtr<IAnalyzerNRTResult>();
		}

		// Convert 16 bit pcm to 32 bit float
		TSampleBuffer<float> FloatSamples((const int16*)InRawWaveData.GetData(), InRawWaveData.Num() / 2, InNumChannels, InSampleRate);

		// Perform and finalize audio analysis.
		Worker->Analyze(FloatSamples.GetArrayView(), Result.Get());
		Worker->Finalize(Result.Get());

		return Result;
	}
}

