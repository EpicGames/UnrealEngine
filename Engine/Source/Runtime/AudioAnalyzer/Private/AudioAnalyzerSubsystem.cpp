// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioAnalyzerSubsystem.h"
#include "AudioAnalyzerModule.h"
#include "Containers/Ticker.h"
#include "Engine/World.h"
#include "Engine/Engine.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioAnalyzerSubsystem)

UAudioAnalyzerSubsystem::UAudioAnalyzerSubsystem()
{
}

UAudioAnalyzerSubsystem::~UAudioAnalyzerSubsystem()
{
	if (TickerHandle.IsValid())
	{
		FTSTicker::RemoveTicker(TickerHandle);
		TickerHandle.Reset();
	}
	AudioAnalyzers.Reset();
}

UAudioAnalyzerSubsystem* UAudioAnalyzerSubsystem::Get()
{
	if (GEngine)
	{
		return GEngine->GetEngineSubsystem<UAudioAnalyzerSubsystem>();
	}
	return nullptr;
}

bool UAudioAnalyzerSubsystem::Tick(float DeltaTime)
{
	AUDIO_ANALYSIS_LLM_SCOPE

	// Loop through all analyzers and if they're ready to analyze, do it
	for (UAudioAnalyzer* Analyzer : AudioAnalyzers)
	{
		if (Analyzer)
		{
			if (Analyzer->IsReadyForAnalysis())
			{
				if (Analyzer->DoAnalysis())
				{
					Analyzer->BroadcastResults();
				}
			}
		}
	}
	return true;
}

void UAudioAnalyzerSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);
}

void UAudioAnalyzerSubsystem::Deinitialize()
{
	AUDIO_ANALYSIS_LLM_SCOPE

	// Release our audio analyzers
	AudioAnalyzers.Reset();
}

void UAudioAnalyzerSubsystem::RegisterAudioAnalyzer(UAudioAnalyzer* InAnalyzer)
{
	AUDIO_ANALYSIS_LLM_SCOPE

	if (!TickerHandle.IsValid())
	{
		TickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateUObject(this, &UAudioAnalyzerSubsystem::Tick), 0.0f);
	}
	AudioAnalyzers.AddUnique(InAnalyzer);
}

void UAudioAnalyzerSubsystem::UnregisterAudioAnalyzer(UAudioAnalyzer* InAnalyzer)
{
	AUDIO_ANALYSIS_LLM_SCOPE

	AudioAnalyzers.Remove(InAnalyzer);
	if (AudioAnalyzers.IsEmpty() && TickerHandle.IsValid())
	{
		FTSTicker::RemoveTicker(TickerHandle);
		TickerHandle.Reset();
	}
}
