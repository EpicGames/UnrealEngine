// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "IAudioAnalyzerInterface.h"
#include "Templates/UniquePtr.h"
#include "UObject/NameTypes.h"

namespace Audio
{

	IAnalyzerFactory* GetAnalyzerFactory(FName InFactoryName);

	/** FAnalyzer
	 *
	 * FAnalyzer provides a simplified interface for performing analysis on audio buffers from the same AudioAnalyzer.
	 */
	class FAnalyzerFacade
	{
		public:
			/**
			 * Create an FAnalyzerBatch with the analyzer settings and factory name.
			 */
			AUDIOANALYZER_API FAnalyzerFacade(TUniquePtr<IAnalyzerSettings> InSettings, IAnalyzerFactory* InFactory);

			/**
			 * Analyze the audio buffer.
			 */
			AUDIOANALYZER_API TUniquePtr<IAnalyzerResult> AnalyzeAudioBuffer(const TArray<float>& InAudioBuffer, int32 InNumChannels, float InSampleRate, TSharedPtr<IAnalyzerControls> InControls);

		private:

			TUniquePtr<IAnalyzerSettings> Settings;
			TUniquePtr<IAnalyzerWorker> Worker;

			IAnalyzerFactory* Factory;
			FName FactoryName;
	};
}
