// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/LowLevelMemTracker.h"
#include "Logging/LogMacros.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

DECLARE_LOG_CATEGORY_EXTERN(LogAudioAnalyzer, Log, All);

LLM_DECLARE_TAG_API(Audio_Analysis, AUDIOANALYZER_API);
// Convenience macro for Audio_Analysis LLM scope to avoid misspells.
#define AUDIO_ANALYSIS_LLM_SCOPE LLM_SCOPE_BYTAG(Audio_Analysis);

class FAudioAnalyzerModule : public IModuleInterface
{
	public:
	/** IModuleInterface implementation */
	AUDIOANALYZER_API void StartupModule();
	AUDIOANALYZER_API void ShutdownModule();
};
