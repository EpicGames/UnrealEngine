// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "IAudioAnalyzerNRTInterface.h"
#include "Templates/UniquePtr.h"
#include "UObject/NameTypes.h"

namespace Audio
{

	IAnalyzerNRTFactory* GetAnalyzerNRTFactory(FName InFactoryName);

	/** FAnalyzerNRTFacade
	 *
	 * FAnalyzerNRTFacade provides a simplified interface for running 
	 * analyzer factories over complete audio resources.
	 */
	class FAnalyzerNRTFacade
	{
		public:
			/**
			 * Create an FAnalyzerNRTFacade with the analyzer settings and factory name.
			 */
			AUDIOANALYZER_API FAnalyzerNRTFacade(TUniquePtr<IAnalyzerNRTSettings> InSettings, const FName& InFactoryName);

			/**
			 * Analyze an entire PCM16 encoded audio object.  Audio for the entire sound should be contained within InRawWaveData.
			 */
			AUDIOANALYZER_API TUniquePtr<IAnalyzerNRTResult> AnalyzePCM16Audio(const TArray<uint8>& InRawWaveData, int32 InNumChannels, float InSampleRate);

		private:

			TUniquePtr<IAnalyzerNRTSettings> Settings;
			FName FactoryName;
	};
}
