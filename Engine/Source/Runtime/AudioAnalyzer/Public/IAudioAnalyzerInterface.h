// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Features/IModularFeature.h"

namespace Audio
{
	/** FAnalyzerParameters
	 *
	 * These parameters are pass to an IAnalyzerFactory when creating
	 * a new IAnalyzerWorker or IAnalyzerResult
	 */
	struct FAnalyzerParameters
	{
	public:
		int32 SampleRate;
		int32 NumChannels;

		FAnalyzerParameters(int32 InSampleRate = 0, int32 InNumChannels = 0)
			: SampleRate(InSampleRate)
			, NumChannels(InNumChannels)
		{}
	};

	/** IAnalyzerSettings
	 *
	 * This interface defines the required methods for real-time
	 * analyzer settings.
	 */
	class IAnalyzerSettings
	{
	public:
		virtual ~IAnalyzerSettings() = default;
	};

	/** IAnalyzerResult
	 *
	 * This interface defines the required methods for real-time
	 * analyzer results.
	 */
	class IAnalyzerResult
	{
	public:
		virtual ~IAnalyzerResult() = default;
	};

	/** IAnalyzerControls
	 *
	 * This interface defines the controls we want to link to 
	 * the analyzer in order to do real-time parameter changes
	 */
	class IAnalyzerControls
	{
	public:
		virtual ~IAnalyzerControls() = default;
	};

	// An audio analyzer worker
	class IAnalyzerWorker
	{
	public:
		virtual ~IAnalyzerWorker() = default;

		virtual void Analyze(TArrayView<const float> InAudio, IAnalyzerResult* OutResult) = 0;

		virtual void SetControls(TSharedPtr<IAnalyzerControls> InAnalyzerControls) {}
		virtual void ClearControls() {}
	};

	// Audio analyzer factory. Primarily used for creating a worker.
	class IAnalyzerFactory : public IModularFeature
	{
	public:
		virtual ~IAnalyzerFactory() = default;

		// Supplied unique name of IAnalyzerFactory to enable querying of added 
		// analyzer factories
		static FName GetModularFeatureName()
		{
			static FName AudioExtFeatureName = FName("AudioAnalyzerPlugin");
			return AudioExtFeatureName;
		}

		// Name of specific analyzer type.
		virtual FName GetName() const = 0;

		// Human readable name of analyzer.
		virtual FString GetTitle() const = 0;

		// Create a new result.
		virtual TUniquePtr<IAnalyzerResult> NewResult() const = 0;

		// Convenience function to create a new shared result by calling NewResult.
		template<ESPMode Mode = ESPMode::ThreadSafe>
		TSharedPtr<IAnalyzerResult, Mode> NewResultShared() const
		{
			TUniquePtr<Audio::IAnalyzerResult> Result = NewResult();

			return TSharedPtr<Audio::IAnalyzerResult, Mode>(Result.Release());
		}

		// Create a new worker.
		virtual TUniquePtr<IAnalyzerWorker> NewWorker(const FAnalyzerParameters& InParams, const IAnalyzerSettings* InSettings) const = 0;
	};
}



