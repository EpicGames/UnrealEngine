// Copyright Epic Games, Inc. All Rights Reserved.

/** Plugin interface for Non Real-Time (NRT) Audio Analyzers. */

#pragma once

#include "CoreMinimal.h"
#include "Features/IModularFeature.h"
#include "Serialization/BitReader.h"
#include "Serialization/BitWriter.h"

namespace Audio
{
	
	/** FAnalyzerNRTParameters
	 *
	 * These parameters are pass to an IAnalyzerNRTFactory when creating
	 * a new IAnalyzerNRTWorker or IAnalyzerNRTResult
	 */
	struct FAnalyzerNRTParameters
	{
		public:
			float SampleRate;
			int32 NumChannels;

		FAnalyzerNRTParameters(float InSampleRate=0.0f, int32 InNumChannels=0)
		:	SampleRate(InSampleRate)
		,	NumChannels(InNumChannels)
		{}
	};

	/** IAnalyzerNRTSettings
	 *
	 * This interface defines the required methods for non-real-time
	 * analyzer settings.
	 */
	class IAnalyzerNRTSettings
	{
		public:
			virtual ~IAnalyzerNRTSettings() = default;
	};

	/** IAnalyzerNRTResult
	 *
	 * This interface defines the required methods for non-real-time
	 * analyzer results.
	 */
	class IAnalyzerNRTResult
	{
	public:
		virtual ~IAnalyzerNRTResult() = default;

		// This is used to define how to serialize this instance of results for remote profiling.
		virtual void Serialize(FArchive& Archive) = 0;

		// This virtual can be overridden to provide a faster copying scheme than full serialization
		// when analyzing non-remote targets. If not overridden, this function will use the Serialize call.
		virtual void CopyFrom(IAnalyzerNRTResult* SourceResult)
		{
			FBitWriter SerializedResult;
			SourceResult->Serialize(SerializedResult);
			FBitReader DeserializedResult = FBitReader(SerializedResult.GetData(), SerializedResult.GetNumBytes() * 8);
			Serialize(DeserializedResult);
		}

		// This must be overridden to return the duration of the original audio analyzed.
		virtual float GetDurationInSeconds() const = 0;
	};

	/** IAnalyzerNRTWorker
	 *
	 * This interface is used to define a class that will handle actual
	 * analysis of a singular audio asset.
	 */
	class IAnalyzerNRTWorker
	{
	public:
		virtual ~IAnalyzerNRTWorker() = default;

		/** 
		 * Perform analysis of an audio stream.
		 * This method may be called multiple times with audio from the same source.
		 */
		virtual void Analyze(TArrayView<const float> InAudio, IAnalyzerNRTResult* OutResult) = 0;

		/** 
		 * Called when analysis of audio asset is complete. 
		 */
		virtual void Finalize(IAnalyzerNRTResult* OutResult) = 0;
	};

	/** IAnalyzerNRTFactory
	 *
	 * This is used to define a non real-time analyzer.
	 */
	class IAnalyzerNRTFactory : public IModularFeature
	{
	public:
		virtual ~IAnalyzerNRTFactory() = default;

		// Supplied unique name of IAnalyzerNRTFactory to enable querying of added 
		// analyzer factories
		static FName GetModularFeatureName()
		{
			static FName AudioExtFeatureName = FName("AudioAnalyzerNRTPlugin");
			return AudioExtFeatureName;
		}

		// Name of specific analyzer type.
		virtual FName GetName() const = 0;

		// Human readable name of analyzer.
		virtual FString GetTitle() const = 0;

		// Create a new result.
		virtual TUniquePtr<IAnalyzerNRTResult> NewResult() const = 0;

		// Convenience function to create a new shared result by calling NewResult.
		template<ESPMode Mode = ESPMode::ThreadSafe>
		TSharedPtr<IAnalyzerNRTResult, Mode> NewResultShared() const
		{
			TUniquePtr<Audio::IAnalyzerNRTResult> Result = NewResult();

			return TSharedPtr<Audio::IAnalyzerNRTResult, Mode>(Result.Release());
		}

		// Create a new worker.
		virtual TUniquePtr<IAnalyzerNRTWorker> NewWorker(const FAnalyzerNRTParameters& InParams, const IAnalyzerNRTSettings* InSettings) const = 0;
	};
}

